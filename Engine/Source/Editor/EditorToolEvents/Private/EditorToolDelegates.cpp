// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorToolDelegates.h"

FEditorToolDelegates::FEditorToolStarted	FEditorToolDelegates::OnEditorToolStarted;
FEditorToolDelegates::FSendEditorFocusSummary	FEditorToolDelegates::OnSendEditorFocusSummary;
