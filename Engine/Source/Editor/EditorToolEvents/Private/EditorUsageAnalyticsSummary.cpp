// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUsageAnalyticsSummary.h"

#include "EditorToolDelegates.h"
#include "Framework/Application/SlateApplication.h"
// includes for editor focus summary test
#include "Framework/Application/SlateUser.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SWindow.h"

// json includes
#include "Dom/JsonObject.h"

#if WITH_EDITOR

namespace EditorUsageAnalyticsSummaryHelpers
{
	bool GetFocusedTabIdentifierIfInFocusPath(TSharedPtr<const FSlateUser> SlateUser, FString& OutTabIdentifier, FString &OutWindowId)
	{
		if (!SlateUser.IsValid() || !SlateUser->HasValidFocusPath())
		{
			return false;
		}

		const FWeakWidgetPath& WeakFocusPath = SlateUser->GetWeakFocusPath();

		#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
		if (WeakFocusPath.Widgets.Num() > 0 && WeakFocusPath.Widgets[0].IsValid())
		{
			OutWindowId = FString::FromInt(WeakFocusPath.Widgets[0].Pin()->GetId());
		}
		#else
		// It's not perfect but we can at least use the title to uniquely distinguish windows. 
		if (WeakFocusPath.Widgets.Num() > 0 && WeakFocusPath.Widgets[0].IsValid())
		{
			TSharedPtr<SWidget> WidgetInPath = WeakFocusPath.Widgets[0].Pin();

			if (WidgetInPath->GetTypeAsString() == TEXT("SWindow"))
			{
				SWindow* WindowWidget = reinterpret_cast<SWindow*>(WidgetInPath.Get());

				if (WindowWidget != nullptr)
				{
					OutWindowId = WindowWidget->GetTitle().ToString();
				}
				else
				{
					OutWindowId = FString::FromInt(0);
				}
			}
		}
		else
		{
			// if we can't use the widget ID we have to lump all of our focus data as if they are in a single window
			OutWindowId = FString::FromInt(0);
		}
		#endif

		// intentionally don't check 0 index as that'll be the window
		for (int i = WeakFocusPath.Widgets.Num() - 1; i > 0; --i)
		{
			TWeakPtr<SWidget> WidgetInPathWeakPtr = WeakFocusPath.Widgets[i];

			if (!WidgetInPathWeakPtr.IsValid())
			{
				continue;
			}

			TSharedPtr<SWidget> WidgetInPath = WidgetInPathWeakPtr.Pin();
			// docking tab stack is the part of the docking stuff that shows up in focus paths
			if (WidgetInPath->GetTypeAsString() == TEXT("SDockingTabStack"))
			{
				SDockingTabStack* TabStackWidget = reinterpret_cast<SDockingTabStack*>(WidgetInPath.Get());

				if (TabStackWidget != nullptr)
				{
					TSharedPtr<SDockTab> TabWidget = TabStackWidget->GetSelectedTabFromWell();

					if (TabWidget.IsValid())
					{
						OutTabIdentifier = TabWidget->GetLayoutIdentifier().ToString();

						return true;
					}

				}
			}
		}

		return false;
	}

	static const double DefaultFocusedSampleDelay = 3.0;
}

FEditorUsageAnalyticsSummary& FEditorUsageAnalyticsSummary::Get()
{
	static FEditorUsageAnalyticsSummary Summary = FEditorUsageAnalyticsSummary();
	return Summary;
}

FEditorUsageAnalyticsSummary::FEditorUsageAnalyticsSummary() : TimeOfLastUpdate(FPlatformTime::Seconds())
{
	double Delay = 0.0;
	if (!GConfig->GetDouble(TEXT("EditorUsageAnalytics"), TEXT("FocusedSampleDelay"), Delay, GEditorIni))
	{
		Delay = EditorUsageAnalyticsSummaryHelpers::DefaultFocusedSampleDelay;
	}

	UpdatePeriod.FromSeconds(Delay);
}

void FEditorUsageAnalyticsSummary::Start()
{
	bActive = true;
	FSlateApplication::Get().GetLastUserInteractionTimeUpdateEvent().AddRaw(this, &FEditorUsageAnalyticsSummary::OnSlateUserInteraction);
}

void FEditorUsageAnalyticsSummary::Shutdown()
{
	FSlateApplication::Get().GetLastUserInteractionTimeUpdateEvent().RemoveAll(this);
	bActive = false;
}

void FEditorUsageAnalyticsSummary::OnSlateUserInteraction(double DeltaTime)
{
	double CurrentTimeSeconds = FPlatformTime::Seconds();

	if (CurrentTimeSeconds >= NextUpdateTimeSeconds)
	{
		UpdateUserFocusTimes(CurrentTimeSeconds);
		// update when we want to next update our usage summary statistics. Get updated platform time as updating could have taken some time
		NextUpdateTimeSeconds = FPlatformTime::Seconds() + UpdatePeriod.GetTotalSeconds();
	}
}

void FEditorUsageAnalyticsSummary::UpdateUserFocusTimes(double CurrTimeSecs)
{
	// This can get called from a console control handler thread. Getting the SlateApplication is only valid on certain threads.
	if (!IsInGameThread() && !IsInSlateThread() && !IsInAsyncLoadingThread() && !IsInParallelLoadingThread())
	{
		return;
	}

	// Make sure we have a slate application to query
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	FSlateApplication& SlateApplication = FSlateApplication::Get();

	TSharedPtr<const FSlateUser> CurrentUser = SlateApplication.GetCursorUser();

	double PrevTimeOfLastUpdate = TimeOfLastUpdate.load();

	// we want to ensure only a single thread performs this update
	if (TimeOfLastUpdate.compare_exchange_strong(PrevTimeOfLastUpdate, CurrTimeSecs))
	{
		FString TabIdentifier;
		FString WindowId;
		if (EditorUsageAnalyticsSummaryHelpers::GetFocusedTabIdentifierIfInFocusPath(CurrentUser, TabIdentifier, WindowId))
		{
			float DeltaAsFloat = static_cast<float>(CurrTimeSecs - PrevTimeOfLastUpdate);

			{
				UE::TScopeLock LockAroundData(StoreLock);

				FFocusedWindowAnalyticData& WindowData = FocusedWindows.FindOrAdd(WindowId);

				if (!WindowData.FocusedTabs.Contains(TabIdentifier))
				{
					WindowData.FocusedTabs.Add(TabIdentifier, 0);
				}
				else
				{
					WindowData.FocusedTabs[TabIdentifier] += DeltaAsFloat;
				}
			}
		}
	}
}

TSharedPtr<FJsonObject> FEditorUsageAnalyticsSummary::GetSummaryDataAsJsonObject(bool bReset)
{
	TSharedPtr<FJsonObject> SummaryAsJsonObject = MakeShareable(new FJsonObject);

	TArray<TSharedPtr<FJsonValue>> ListOfWindows;

	{
		UE::TScopeLock LockAroundFocusData(StoreLock);

		for (const TPair<FString, FFocusedWindowAnalyticData>& WindowDataPair : FocusedWindows)
		{
			TSharedPtr<FJsonObject> WindowEntry = MakeShareable(new FJsonObject);

			// NOTE: Leaving this out as this can be the window title and don't want to worry about EGPI or illegal characters
			//WindowEntry->SetStringField(TEXT("WindowID"), WindowDataPair.Key);
			TArray<TSharedPtr<FJsonValue>> ListOfTabs;
			for (const TPair<FString, double> &TabPair : WindowDataPair.Value.FocusedTabs)
			{
				TSharedPtr<FJsonObject> TabEntry = MakeShareable(new FJsonObject);

				TabEntry->SetStringField(TEXT("TabName"), TabPair.Key);
				TabEntry->SetNumberField(TEXT("FocusTime"), TabPair.Value);

				ListOfTabs.Add(TSharedPtr<FJsonValue>(new FJsonValueObject(TabEntry)));
			}

		
			WindowEntry->SetArrayField(TEXT("Tabs"), ListOfTabs);

			ListOfWindows.Add(TSharedPtr<FJsonValue>(new FJsonValueObject(WindowEntry)));
		}

		if (bReset)
		{
			FocusedWindows.Reset();
		}
	}

	SummaryAsJsonObject->SetArrayField(TEXT("Windows"), ListOfWindows);

	return SummaryAsJsonObject;
}
#endif// WITH_EDITOR
