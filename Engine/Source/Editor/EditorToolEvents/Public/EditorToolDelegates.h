// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Struct of static delegates for editor tools. Delegates intended to be used by projects to send tool telemetry events
 */

#pragma once

#include "Delegates/Delegate.h"
#include "Dom/JsonObject.h"
#include "Templates/SharedPointer.h"

struct FEditorToolDelegates
{

public:
	/**Broadcasted for editor tools OnLoad or OnInitialize, if available to notify when tool used */
	DECLARE_MULTICAST_DELEGATE_OneParam(FEditorToolStarted, const FString &/*class name*/);
	static EDITORTOOLEVENTS_API FEditorToolStarted OnEditorToolStarted;
	
	/**Broadcasted when the editor is closed and sends a summary of widget focus data */
	DECLARE_MULTICAST_DELEGATE_OneParam(FSendEditorFocusSummary, TSharedPtr<FJsonObject>/*json summary of focus data*/);
	static EDITORTOOLEVENTS_API FSendEditorFocusSummary OnSendEditorFocusSummary;
};
