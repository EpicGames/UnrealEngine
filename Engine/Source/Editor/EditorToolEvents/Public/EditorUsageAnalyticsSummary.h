// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#if WITH_EDITOR

struct FFocusedWindowAnalyticData
{
	// Name, time in seconds focused
	TMap<FString, double> FocusedTabs;
	uint32 WindowId;
};

class FJsonObject;

/*
*	Collects editor events we wish to send as a summary that are not critical enough to be in our summary store.
*/
class FEditorUsageAnalyticsSummary : FNoncopyable
{
public:
	EDITORTOOLEVENTS_API static FEditorUsageAnalyticsSummary& Get();

	EDITORTOOLEVENTS_API FEditorUsageAnalyticsSummary();

	EDITORTOOLEVENTS_API void Start();

	EDITORTOOLEVENTS_API void Shutdown();

	// Get's the current usage summary. If bReset is true, clears out current summary. (useful when collecting summaries per project instead of session)
	EDITORTOOLEVENTS_API TSharedPtr<FJsonObject> GetSummaryDataAsJsonObject(bool bReset = false);

	// Returns true if start has been called. Makes it so we can still check the summary even if the CVAR to turn it off is true or otherwise disabled.
	bool IsActive() { return bActive; }

	virtual ~FEditorUsageAnalyticsSummary() = default;

private:

	void OnSlateUserInteraction(double DeltaTime);
	void UpdateUserFocusTimes(double CurrTimeSecs);

	TMap<FString, FFocusedWindowAnalyticData> FocusedWindows;

	// Time since we last updated our focus stats
	std::atomic<double> TimeOfLastUpdate;

	//Slate interaction and shutting down can come from different threads so we need a log for FocusedWindow data
	mutable FTransactionallySafeCriticalSection StoreLock;

	//Period we want to wait between focus time updates (some events might lead us to update sooner)
	FTimespan UpdatePeriod;

	//The next time the session properties should be persisted to disk.
	double NextUpdateTimeSeconds = 0.0;

	bool bActive = false;
};

#endif